{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Sequencer Mixed Control Rig",
	"Description" : "System for using the Anim Mixer to mix control rig tracks",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins": [
		{
			"Name": "MovieSceneAnimMixer",
			"Enabled": true
		},
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "UAFControlRig",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "MovieSceneMixedControlRig",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "MovieSceneMixedControlRigEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}
