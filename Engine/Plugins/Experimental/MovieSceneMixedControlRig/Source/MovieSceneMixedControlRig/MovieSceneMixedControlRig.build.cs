// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieSceneMixedControlRig : ModuleRules
	{
		public MovieSceneMixedControlRig(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"DeveloperSettings",
					"MovieScene",
					"MovieSceneTracks",
					"AnimGraphRuntime",
					"MovieSceneAnimMixer",
					"ControlRig",
					"UAFControlRig",
					"RigVM",
					"AnimationCore",
					"UAF",
					"UAFAnimGraph"
				}
			);
		}
	}
}