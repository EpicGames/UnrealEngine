// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMixedControlRigModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "IMovieSceneModule.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneMirroringDecoration.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"

DEFINE_LOG_CATEGORY(LogMovieSceneMixedControlRig);

#define LOCTEXT_NAMESPACE "FMixedControlRigModule"

namespace UE::MovieScene
{

void FMovieSceneMixedControlRigModule::StartupModule()
{
	IMovieSceneModule::Get().RegisterCompatibleDecoration(
		UMovieSceneControlRigParameterTrack::StaticClass(),
		UMovieSceneMirroringDecoration::StaticClass());

	IMovieSceneModule::Get().RegisterCompatibleDecoration(
		UMovieSceneControlRigParameterTrack::StaticClass(),
		UMovieSceneRootMotionSettingsDecoration::StaticClass());
}

void FMovieSceneMixedControlRigModule::ShutdownModule()
{
	if (IMovieSceneModule::IsAvailable() && UObjectInitialized())
	{
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(
			UMovieSceneControlRigParameterTrack::StaticClass(),
			UMovieSceneMirroringDecoration::StaticClass());

		IMovieSceneModule::Get().UnregisterCompatibleDecoration(
			UMovieSceneControlRigParameterTrack::StaticClass(),
			UMovieSceneRootMotionSettingsDecoration::StaticClass());
	}
}

}

IMPLEMENT_MODULE(UE::MovieScene::FMovieSceneMixedControlRigModule, MovieSceneMixedControlRig)

#undef LOCTEXT_NAMESPACE