// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneMixedControlRigSystem.h"

#include "Async/TaskGraphInterfaces.h"
#include "Decorations/MovieSceneScalingAnchors.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "AnimMixerComponentTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "MovieScene.h"
#include "MovieSceneTracksComponentTypes.h"
#include "PreAnimatedState/MovieScenePreAnimatedComponentTransformStorage.h"
#include "Sequencer/MovieSceneControlRigComponentTypes.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigSystem.h"
#include "IControlRigObjectBinding.h"
#include "ControlRigTask.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "ControlRigObjectBinding.h"
#include "Rigs/FKControlRig.h"
#include "MovieSceneAnimationMixerTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMixedControlRigSystem)


namespace UE::MovieScene
{

bool GMovieSceneControlRigAnimMixerEnabled = true;
FAutoConsoleVariableRef CVarMovieSceneControlRigAnimMixerEnabled(
	TEXT("Sequencer.ControlRigAnimMixer.Enabled"),
	GMovieSceneControlRigAnimMixerEnabled,
	TEXT("(Default: true) Controls whether the new Anim Mixer library is used for control rig evaluation.\n"),
	ECVF_Default);

struct FEncounteredTrackTaskData
{
	FMovieSceneEntityID EntityID;
	bool bHasWeight = false;
};

// Adds required anim mixer components onto control rig track entities
struct FControlRigMixerMutation : IMovieSceneEntityMutation
{
	FBuiltInComponentTypes* BuiltInComponents;
	FMovieSceneTracksComponentTypes* TrackComponents;
	FAnimMixerComponentTypes* AnimMixerComponents;
	FControlRigComponentTypes* ControlRigComponents;
	UMovieSceneEntitySystemLinker* Linker = nullptr;

	// Track encountered tracks to prevent duplicate task creation
	// Maps Track -> task data for the "active" entity
	// Using Track (not ControlRig) as key because multiple tracks on different
	// layers can use the same UControlRig, but each track should have its own task
	mutable TMap<UMovieSceneControlRigParameterTrack*, FEncounteredTrackTaskData> EncounteredTrackTasks;

	// Entity IDs that need their tasks cleared after the mutation completes
	// (entities that were demoted due to weight priority)
	mutable TArray<FMovieSceneEntityID> EntitiesToClearTask;

	// Entity IDs that need the SkipPoseWeight tag added after the mutation completes
	// (entities with non-absolute blend type where weight is already applied to parameters)
	mutable TArray<FMovieSceneEntityID> EntitiesToAddSkipPoseWeight;

	// Entity IDs that need the PreBakeRootMotion tag added after the mutation completes.
	// CR eval tasks read the previous pose's bone[0] to position their hierarchy, so the
	// mixer must bake accumulated root motion onto bone[0] before the eval task runs.
	mutable TArray<FMovieSceneEntityID> EntitiesToAddPreBakeRootMotion;

	FControlRigMixerMutation(UMovieSceneEntitySystemLinker* InLinker)
	{
		BuiltInComponents = FBuiltInComponentTypes::Get();
		TrackComponents = FMovieSceneTracksComponentTypes::Get();
		AnimMixerComponents = FAnimMixerComponentTypes::Get();
		ControlRigComponents = FControlRigComponentTypes::Get();
		Linker = InLinker;
	}

	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
	{
		InOutEntityComponentTypes->SetAll(
		{
			TrackComponents->Tags.AnimMixerPoseProducer,
			AnimMixerComponents->Priority,
			AnimMixerComponents->Target,
			AnimMixerComponents->Task,
			AnimMixerComponents->MixerEntry,
			BuiltInComponents->BoundObject,
			BuiltInComponents->BoundObjectKey
		});
	}

	virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
	{
		const int32 Num = Allocation->Num();
		FEntityAllocationWriteContext WriteContext = FEntityAllocationWriteContext::NewAllocation();
		TArrayView<const FMovieSceneEntityID> EntityIDs = Allocation->GetEntityIDs();
		TComponentReader<FControlRigSourceData> ControlRigSourceData = Allocation->ReadComponents(ControlRigComponents->ControlRigSource);
		TComponentReader<FRootInstanceHandle> RootInstanceHandles = Allocation->ReadComponents(BuiltInComponents->RootInstanceHandle);
		TComponentReader<FGuid> ObjectBindingIDs = Allocation->ReadComponents(BuiltInComponents->GenericObjectBinding);
		TOptionalComponentReader<FMovieSceneSequenceID> OptSequenceIDs = Allocation->TryReadComponents(BuiltInComponents->SequenceID);
		TOptionalComponentReader<TObjectPtr<UMovieSceneAnimationMixerLayer>> OptMixerLayers = Allocation->TryReadComponents(AnimMixerComponents->MixerLayer);
		TOptionalComponentReader<double> OptWeightAndEasing = Allocation->TryReadComponents(BuiltInComponents->WeightAndEasingResult);

		TComponentWriter<TInstancedStruct<FMovieSceneMixedAnimationTarget>> OutAnimTargets = Allocation->WriteComponents(AnimMixerComponents->Target, WriteContext);
		TComponentWriter<TSharedPtr<FAnimNextEvaluationTask>> OutTasks = Allocation->WriteComponents(AnimMixerComponents->Task, WriteContext);
		TComponentWriter<TSharedPtr<FMovieSceneAnimMixerEntry>> OutMixerEntries = Allocation->WriteComponents(AnimMixerComponents->MixerEntry, WriteContext);
		TComponentWriter<int32> OutPriorities = Allocation->WriteComponents(AnimMixerComponents->Priority, WriteContext);
		
		TComponentWriter<UObject*> OutBoundObjects = Allocation->WriteComponents(BuiltInComponents->BoundObject, WriteContext);
		TComponentWriter<FObjectKey> OutBoundObjectKeys = Allocation->WriteComponents(BuiltInComponents->BoundObjectKey, WriteContext);
		
		for (int32 Index = 0; Index < Num; ++Index)
		{
			// Set some defaults
			OutTasks[Index] = nullptr;
			OutMixerEntries[Index] = nullptr; 
			OutBoundObjects[Index] = nullptr;
			OutBoundObjectKeys[Index] = nullptr;

			const FControlRigSourceData& ControlRigSource = ControlRigSourceData[Index];
			if (ControlRigSource.Track)
			{
				// Bit of a hack- we resolve the object binding here because the original Control Rig track was never built to work with resolved object bindings
				// but the mixer requires them. We only resolve the first object because control rig does not support multi-object bindings.
				FMovieSceneSequenceID SequenceID = OptSequenceIDs ? OptSequenceIDs[Index] : MovieSceneSequenceID::Root;
				TArrayView<TWeakObjectPtr<>> BoundObjects = Linker->GetInstanceRegistry()->GetInstance(RootInstanceHandles[Index]).GetSharedPlaybackState()->FindBoundObjects(ObjectBindingIDs[Index], SequenceID);
				if (!BoundObjects.IsEmpty())
				{
					UObject* BoundObject = UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding(BoundObjects[0].Get());
					if (BoundObject)
					{
						OutBoundObjects[Index] = BoundObject;
						OutBoundObjectKeys[Index] = BoundObject;

						UWorld* GameWorld = (BoundObject && BoundObject->GetWorld() && BoundObject->GetWorld()->IsGameWorld()) ? BoundObject->GetWorld() : nullptr;
						UControlRig* ControlRig = GameWorld ? ControlRigSource.Track->GetGameWorldControlRig(GameWorld, BoundObject) : ControlRigSource.Track->GetControlRig();

						if (ControlRig)
						{
							const bool bHasWeight = OptWeightAndEasing != nullptr;
							UMovieSceneControlRigParameterTrack* Track = ControlRigSource.Track;

							// Lambda to initialize control rig binding
							auto InitializeControlRigBinding = [](UControlRig* InControlRig, UObject* InBoundObject) -> bool
							{
								bool bNeedsInit = false;
								if (!InControlRig->GetObjectBinding())
								{
									InControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
								}

								if (InControlRig->GetObjectBinding()->GetBoundObject() != FControlRigObjectBinding::GetBindableObject(InBoundObject))
								{
									InControlRig->GetObjectBinding()->BindToObject(InBoundObject);
									bNeedsInit = true;
								}
								return bNeedsInit;
							};

							// Lambda to initialize control rig task
							auto InitializeControlRigTask = [&ControlRigSource, GameWorld](
								TSharedPtr<FAnimNextEvaluationTask>& Task,
								UControlRig* InControlRig,
								UObject* InBoundObject,
								bool bNeedsInit)
							{
								if (TSharedPtr<FAnimNextControlRigTask> ControlRigTask = StaticCastSharedPtr<FAnimNextControlRigTask>(Task))
								{
									TArray<FName> SelectedControls = InControlRig->CurrentControlSelection();

									ControlRigTask->Initialize(bNeedsInit);

									if (bNeedsInit)
									{
										if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(FControlRigObjectBinding::GetBindableObject(InBoundObject)))
										{
											InControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkeletalMeshComponent, false);
											InControlRig->Evaluate_AnyThread();
										}

										if (GameWorld == nullptr && InControlRig->IsA<UFKControlRig>())
										{
											ControlRigSource.Track->ReplaceControlRig(InControlRig, true);
										}

										TArray<FName> NewSelectedControls = InControlRig->CurrentControlSelection();
										if (SelectedControls != NewSelectedControls)
										{
											InControlRig->ClearControlSelection();
											for (const FName& Name : SelectedControls)
											{
												InControlRig->SelectControl(Name, true);
											}
										}
									}
								}
							};

							// Check if we've already created a task for this Track
							const FMovieSceneEntityID EntityID = EntityIDs[Index];
							FEncounteredTrackTaskData* ExistingData = EncounteredTrackTasks.Find(Track);
							if (ExistingData)
							{
								if (ExistingData->bHasWeight)
								{
									// Previous entity has weight - it keeps priority
									// Current entity doesn't get a task
									OutTasks[Index] = nullptr;
								}
								else if (bHasWeight)
								{
									// Previous has NO weight, current DOES - demote previous, promote current
									// Add previous entity to the list to clear after mutation completes
									EntitiesToClearTask.Add(ExistingData->EntityID);

									// Create new task for current entity
									bool bNeedsInit = InitializeControlRigBinding(ControlRig, BoundObject);

									FAnimNextControlRigTaskParams Params;
									Params.bConsumesPreviousPose = false;
									Params.ControlRig = ControlRig;

									OutTasks[Index] = MakeShared<FAnimNextControlRigTask>(MoveTemp(Params));
									InitializeControlRigTask(OutTasks[Index], ControlRig, BoundObject, bNeedsInit);

									// Update tracking
									ExistingData->EntityID = EntityID;
									ExistingData->bHasWeight = true;
								}
								else
								{
									// Neither has weight - previous keeps priority
									OutTasks[Index] = nullptr;
								}
							}
							else
							{
								// First time seeing this track - create new task
								bool bNeedsInit = InitializeControlRigBinding(ControlRig, BoundObject);

								FAnimNextControlRigTaskParams Params;
								Params.bConsumesPreviousPose = false;
								Params.ControlRig = ControlRig;

								OutTasks[Index] = MakeShared<FAnimNextControlRigTask>(MoveTemp(Params));
								InitializeControlRigTask(OutTasks[Index], ControlRig, BoundObject, bNeedsInit);

								// Track this track
								FEncounteredTrackTaskData NewData;
								NewData.EntityID = EntityID;
								NewData.bHasWeight = bHasWeight;
								EncounteredTrackTasks.Add(Track, NewData);
							}
						}

						// Always set up mixer components (even for entities without tasks)
						// If we haven't had a mixer layer set, then set default anim target/priority
						if (!OptMixerLayers || !OptMixerLayers[Index])
						{
							// Set a default target- Automatic will resolve to the appropriate place, or the Anim Mixer track will override it.
							OutAnimTargets[Index] = TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make();
							// Convert the "Priority Order" on the control rig track to the anim mixer's concept of priority. We do it this way also
							// to ensure all control rigs (by default anyway) happen after the Animation track.
							// Control rig tracks start at priority 100 and then decrease from there, with larger numbers evaluating first
							// Anim mixer uses priority to mean that larger priorities go later (as they will overwrite lower priority poses)
							// We use an arbitrary high priority to try to ensure that priority values set by control rig users still will evaluate after animation track
							const int32 HighPriority = 1000000;
							OutPriorities[Index] = 1 + (HighPriority - ControlRigSource.Track->GetPriorityOrder());
						}

						// Track entities that need SkipPoseWeight tag (non-absolute blend types).
						// For non-absolute blends (Additive, Relative, Override), weight is already applied
						// to control parameters during ECS blending, so we skip pose weight to avoid double-weighting.
						const bool bIsAbsoluteBlend = ControlRigSource.Section.IsValid() && ControlRigSource.Section->GetBlendType() == EMovieSceneBlendType::Absolute;
						if (!bIsAbsoluteBlend)
						{
							const FMovieSceneEntityID EntityID = EntityIDs[Index];
							EntitiesToAddSkipPoseWeight.Add(EntityID);
						}

						if (OutTasks[Index])
						{
							EntitiesToAddPreBakeRootMotion.Add(EntityIDs[Index]);
						}
					}
				}
			}
		}
	}
};

/** ------------------------------------------------------------------------- */
/** Task for gathering base control rigs and setting up their evaluation tasks. */
struct FUpdateBaseControlRigs
{

	const FInstanceRegistry* InstanceRegistry;

	FUpdateBaseControlRigs(UMovieSceneEntitySystemLinker* InLinker)
		: InstanceRegistry(InLinker->GetInstanceRegistry())
	{
	}

	void ForEachEntity(
		const FInstanceHandle& InstanceHandle,
		UObject* BoundObject, 
		const FControlRigSourceData& ControlRigSource,
		TSharedPtr<FAnimNextEvaluationTask>& EvalTask) const
	{
		if (BoundObject)
		{
			UWorld* GameWorld = (BoundObject && BoundObject->GetWorld() && BoundObject->GetWorld()->IsGameWorld()) ? BoundObject->GetWorld() : nullptr;
			UMovieSceneControlRigParameterTrack* Track = ControlRigSource.Track;

			if (Track == nullptr)
			{
				return;
			}

			UControlRig* ControlRig = GameWorld ? Track->GetGameWorldControlRig(GameWorld, BoundObject) : Track->GetControlRig();

			if (ControlRig == nullptr)
			{
				return;
			}

			UObject* ControlRigBoundObject = ControlRig->GetObjectBinding() != nullptr ? ControlRig->GetObjectBinding()->GetBoundObject() : nullptr;
			if (!ControlRigBoundObject)
			{
				return;
			}

			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRigBoundObject))
			{			
				// Get the full context, so we can get both the current and previous evaluation times.
				const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandle);
				const FMovieSceneContext& Context = SequenceInstance.GetContext();

				if (TSharedPtr<FAnimNextControlRigTask> ControlRigTask = StaticCastSharedPtr<FAnimNextControlRigTask>(EvalTask))
				{
#if UE_ENABLE_DEBUG_DRAWING
					ControlRigTask->SetComponentTransform(SkeletalMeshComponent->GetComponentTransform());
#endif
					if (ControlRigTask->GetControlRig())
					{
						ControlRigTask->GetControlRig()->SetDeltaTime(Context.GetFrameRate().AsSeconds(Context.GetDelta()));
					}
				}
			}
		}
	}
};


} // namespace UE::MovieScene

UMovieSceneMixedControlRigSystem::UMovieSceneMixedControlRigSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();

	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;
	RelevantComponent = ControlRigComponents->BaseControlRigEvalData;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get(); 

		DefineComponentConsumer(GetClass(), ControlRigComponents->BaseControlRigEvalData);
		DefineComponentConsumer(GetClass(), ControlRigComponents->ControlRigSource);

		DefineImplicitPrerequisite(GetClass(), UMovieSceneControlRigParameterEvaluatorSystem::StaticClass());

		DefineComponentProducer(GetClass(), TrackComponents->Tags.AnimMixerPoseProducer);

		DefineComponentProducer(GetClass(), AnimMixerComponents->Task);
		DefineComponentProducer(GetClass(), AnimMixerComponents->Target);
		DefineComponentProducer(GetClass(), AnimMixerComponents->Priority);

	}
}

void UMovieSceneMixedControlRigSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	if (!GMovieSceneControlRigAnimMixerEnabled)
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();
	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		FEntityComponentFilter Filter;
		Filter.All({ BuiltInComponents->RootInstanceHandle, ControlRigComponents->BaseControlRigEvalData, BuiltInComponents->Tags.NeedsLink });
		Filter.None({ BuiltInComponents->Tags.Ignored });

		// Initialize components for control rig mixing
		FControlRigMixerMutation ControlRigMixerMutation(Linker);
		Linker->EntityManager.MutateAll(Filter, ControlRigMixerMutation);

		// Clear tasks for entities that were demoted due to weight priority
		if (ControlRigMixerMutation.EntitiesToClearTask.Num() > 0)
		{
			TSet<FMovieSceneEntityID> EntitiesToClear(ControlRigMixerMutation.EntitiesToClearTask);

			FEntityTaskBuilder()
				.ReadEntityIDs()
				.Write(AnimMixerComponents->Task)
				.FilterAll({ TrackComponents->Tags.AnimMixerPoseProducer })
				.Iterate_PerEntity(&Linker->EntityManager,
					[&EntitiesToClear](FMovieSceneEntityID EntityID, TSharedPtr<FAnimNextEvaluationTask>& Task)
					{
						if (EntitiesToClear.Contains(EntityID))
						{
							Task = nullptr;
						}
					});
		}

		// Add SkipPoseWeight tag to entities with non-absolute blend type
		// (weight is already applied to their parameters during ECS blending)
		for (const FMovieSceneEntityID& EntityID : ControlRigMixerMutation.EntitiesToAddSkipPoseWeight)
		{
			Linker->EntityManager.AddComponent(EntityID, AnimMixerComponents->Tags.SkipPoseWeight);
		}

		// Add PreBakeRootMotion tag to CR entities with eval tasks so the mixer brackets
		// their eval task with bake-into-bone[0] before and extract-or-reset after.
		for (const FMovieSceneEntityID& EntityID : ControlRigMixerMutation.EntitiesToAddPreBakeRootMotion)
		{
			Linker->EntityManager.AddComponent(EntityID, AnimMixerComponents->Tags.PreBakeRootMotion);
		}
	}
}

void UMovieSceneMixedControlRigSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;	
	
	if (!GMovieSceneControlRigAnimMixerEnabled)
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();

	// Gather all skel anim section entities that have been marked with the AnimMixerPoseProducer tag
	FTaskID UpdateTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->BoundObject)
		.Read(ControlRigComponents->ControlRigSource)
		.Write(AnimMixerComponents->Task)
		.FilterAll({ TrackComponents->Tags.AnimMixerPoseProducer, ControlRigComponents->BaseControlRigEvalData })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Schedule_PerEntity<FUpdateBaseControlRigs>(&Linker->EntityManager, TaskScheduler, Linker);
}

