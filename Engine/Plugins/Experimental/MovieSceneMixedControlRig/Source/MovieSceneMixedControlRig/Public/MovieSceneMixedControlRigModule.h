// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "HAL/IConsoleManager.h"

MOVIESCENEMIXEDCONTROLRIG_API DECLARE_LOG_CATEGORY_EXTERN(LogMovieSceneMixedControlRig, Log, All);

namespace UE::MovieScene
{
class FMovieSceneMixedControlRigModule : public IModuleInterface
{
	// IModuleInterface interface
	MOVIESCENEMIXEDCONTROLRIG_API virtual void StartupModule() override;
	MOVIESCENEMIXEDCONTROLRIG_API virtual void ShutdownModule() override;
};

}
