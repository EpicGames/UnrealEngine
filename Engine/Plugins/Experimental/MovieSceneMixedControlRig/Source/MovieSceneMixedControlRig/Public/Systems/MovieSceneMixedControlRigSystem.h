// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "MovieSceneMixedControlRigSystem.generated.h"

// System to handle creating evaluation tasks from control rig tracks for the anim mixer.
UCLASS(MinimalAPI)
class UMovieSceneMixedControlRigSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneMixedControlRigSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
};