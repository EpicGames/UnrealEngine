// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieSceneMixedControlRigEditor : ModuleRules
	{
		public MovieSceneMixedControlRigEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"UnrealEd",
					"EditorFramework",
					"LevelEditor",
					"MovieScene",
					"MovieSceneAnimMixer",
					"MovieSceneTools",
					"Sequencer",
					"ControlRig",
					"ControlRigEditor",
				}
			);
		}
	}
}
