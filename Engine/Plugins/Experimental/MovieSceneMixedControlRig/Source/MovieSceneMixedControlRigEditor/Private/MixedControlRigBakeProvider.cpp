// Copyright Epic Games, Inc. All Rights Reserved.

#include "MixedControlRigBakeProvider.h"

#include "BakeToControlRigHelper.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "ISequencer.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"

void FMixedControlRigBakeProvider::BuildBakeToControlRigMenuSection(
	FMenuBuilder& MenuBuilder,
	const FAnimMixerBakeMenuParams& Params)
{
	if (!Params.Skeleton || !Params.bFilterAssetBySkeleton)
	{
		return;
	}

	// Adapt the generic ChildTrackFactory to the CR-specific type expected by BuildBakeToControlRigMenu
	TFunction<UMovieSceneControlRigParameterTrack*(UMovieScene*)> CRTrackFactory =
		[ChildTrackFactory = Params.ChildTrackFactory](UMovieScene* MovieScene) -> UMovieSceneControlRigParameterTrack*
		{
			UMovieSceneTrack* Track = ChildTrackFactory(MovieScene, UMovieSceneControlRigParameterTrack::StaticClass());
			return Cast<UMovieSceneControlRigParameterTrack>(Track);
		};

	// Adapt the generic OnComplete to the CR-specific type.
	// This adapter activates FControlRigEditMode (which was previously duplicated in callers),
	// then calls the caller's generic OnComplete.
	TSharedPtr<ISequencer> SequencerShared = Params.Sequencer;
	TFunction<void(UMovieSceneTrack*, UObject*, bool)> CallerOnComplete = Params.OnComplete;

	TFunction<void(const UE::ControlRig::FBakeToControlRigResult&)> CROnComplete =
		[SequencerShared, CallerOnComplete](const UE::ControlRig::FBakeToControlRigResult& Result)
		{
			if (Result.bSuccess && Result.ControlRig && SequencerShared)
			{
				// Activate ControlRig edit mode and add the baked rig
				FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(
					GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
				if (!ControlRigEditMode)
				{
					GLevelEditorModeTools().ActivateMode(FControlRigEditMode::ModeName);
					ControlRigEditMode = static_cast<FControlRigEditMode*>(
						GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
				}
				if (ControlRigEditMode)
				{
					ControlRigEditMode->AddControlRigObject(Result.ControlRig, SequencerShared);
				}
			}

			// Call the caller's generic completion handler
			if (CallerOnComplete)
			{
				CallerOnComplete(Result.Track, Result.ControlRig, Result.bSuccess);
			}
		};

	// Adapt the PopulateAnimSequence delegate
	UE::ControlRig::FPopulateAnimSequenceDelegate PopulateDelegate;
	PopulateDelegate.BindLambda(
		[PopulateFn = Params.PopulateAnimSequence]
		(UAnimSequence* AnimSeq, UAnimSeqExportOption* Opts, USkeletalMeshComponent* Mesh) -> bool
		{
			return PopulateFn.Execute(AnimSeq, Opts, Mesh);
		});

	// Build a submenu label matching the verb the caller wants displayed ("Bake Layer to Control Rig" etc.).
	FText SubmenuLabelOverride;
	if (!Params.EntryLabelVerb.IsEmpty())
	{
		SubmenuLabelOverride = FText::Format(
			NSLOCTEXT("MixedControlRigBakeProvider", "BakeToControlRigFmt", "{0} to Control Rig"),
			Params.EntryLabelVerb);
	}

	// Delegate to the ControlRigEditor bake helper
	UE::ControlRig::BuildBakeToControlRigMenu(
		MenuBuilder,
		Params.Skeleton,
		*Params.bFilterAssetBySkeleton,
		PopulateDelegate,
		Params.Sequencer,
		Params.ObjectBinding,
		Params.BoundObject,
		Params.SkelMeshComp,
		CRTrackFactory,
		CROnComplete,
		SubmenuLabelOverride);
}
