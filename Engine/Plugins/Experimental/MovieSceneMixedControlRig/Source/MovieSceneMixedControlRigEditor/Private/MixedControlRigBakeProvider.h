// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneAnimMixerBakeProvider.h"

/**
 * Concrete implementation of IMovieSceneAnimMixerBakeProvider.
 * Delegates to UE::ControlRig::BuildBakeToControlRigMenu() and manages
 * FControlRigEditMode activation after a successful bake.
 */
class FMixedControlRigBakeProvider : public IMovieSceneAnimMixerBakeProvider
{
public:
	virtual void BuildBakeToControlRigMenuSection(
		FMenuBuilder& MenuBuilder,
		const FAnimMixerBakeMenuParams& Params) override;
};
