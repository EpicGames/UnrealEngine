// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMixedControlRigEditorModule.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Features/IModularFeatures.h"
#include "IMovieSceneAnimMixerBakeProvider.h"
#include "MixedControlRigBakeProvider.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Systems/MovieSceneAnimMixerSystem.h"

namespace UE::MovieScene
{

UObject* FMixedControlRigLayerResolver::ResolveEditingContextToMixerLayer(
	const UMovieSceneEntitySystemLinker* Linker,
	const UObject* EditingContext) const
{
	const UControlRig* ControlRig = Cast<UControlRig>(EditingContext);
	if (!ControlRig || !Linker)
	{
		return nullptr;
	}

	const UMovieSceneAnimMixerSystem* MixerSystem = Linker->FindSystem<UMovieSceneAnimMixerSystem>();
	if (!MixerSystem)
	{
		return nullptr;
	}

	return MixerSystem->FindMixerLayer([ControlRig](UMovieSceneAnimationMixerLayer* Layer) -> bool
	{
		if (!Layer || !Layer->HasChildTrack())
		{
			return false;
		}
		const UMovieSceneControlRigParameterTrack* CRTrack =
			Cast<UMovieSceneControlRigParameterTrack>(Layer->GetChildTrack());
		return CRTrack && CRTrack->GetControlRig() == ControlRig;
	});
}

void FMovieSceneMixedControlRigEditorModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(
		IMovieSceneEditingContextLayerResolver::GetModularFeatureName(),
		&LayerResolver);

	BakeProvider = MakeUnique<FMixedControlRigBakeProvider>();
	IModularFeatures::Get().RegisterModularFeature(
		IMovieSceneAnimMixerBakeProvider::GetModularFeatureName(), BakeProvider.Get());
}

void FMovieSceneMixedControlRigEditorModule::ShutdownModule()
{
	if (BakeProvider)
	{
		IModularFeatures::Get().UnregisterModularFeature(
			IMovieSceneAnimMixerBakeProvider::GetModularFeatureName(), BakeProvider.Get());
	}

	IModularFeatures::Get().UnregisterModularFeature(
		IMovieSceneEditingContextLayerResolver::GetModularFeatureName(),
		&LayerResolver);
}

} // namespace UE::MovieScene

IMPLEMENT_MODULE(UE::MovieScene::FMovieSceneMixedControlRigEditorModule, MovieSceneMixedControlRigEditor)
