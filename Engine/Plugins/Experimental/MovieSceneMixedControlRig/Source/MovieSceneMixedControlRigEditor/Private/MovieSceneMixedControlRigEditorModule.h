// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMovieSceneEditingContextLayerResolver.h"
#include "Modules/ModuleInterface.h"

class FMixedControlRigBakeProvider;

namespace UE::MovieScene
{

class FMixedControlRigLayerResolver : public IMovieSceneEditingContextLayerResolver
{
public:
	virtual UObject* ResolveEditingContextToMixerLayer(
		const UMovieSceneEntitySystemLinker* Linker,
		const UObject* EditingContext) const override;
};

class FMovieSceneMixedControlRigEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	FMixedControlRigLayerResolver LayerResolver;
	TUniquePtr<FMixedControlRigBakeProvider> BakeProvider;
};

} // namespace UE::MovieScene
